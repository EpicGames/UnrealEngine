// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PluginToolset : ModuleRules
{
	public PluginToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Kismet",
				"PluginUtils",
				"Projects",
				"SourceControl",
				"ToolsetRegistry",
				"UnrealEd",
				// ... add private dependencies that you statically link with here ...	
			}
			);
	}
}
