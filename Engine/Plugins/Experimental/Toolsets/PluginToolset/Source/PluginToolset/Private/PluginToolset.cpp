// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginToolset.h"

#include "Features/IPluginsEditorFeature.h"
#include "Features/EditorFeatures.h"
#include "Features/IModularFeatures.h"
#include "HAL/FileManager.h"
#include "ISourceControlModule.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "PluginDescriptor.h"
#include "PluginReferenceDescriptor.h"
#include "PluginUtils.h"
#include "SourceControlOperations.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PluginToolset)

// --- Helpers ---
namespace UE::PluginToolset::Private
{
	void RaiseError(const FString& Message)
	{
		UKismetSystemLibrary::RaiseScriptError(	FString::Printf(TEXT("PluginToolset: %s"), *Message));
	}

	// Mirrors SPluginTile::GetAuthoringButtonsVisibility: returns false for engine plugins in
	// installed engine builds, and for non-Mod plugins in any installed build.
	bool CanEditPluginDescriptor(const IPlugin& Plugin)
	{
		if (FApp::IsEngineInstalled() && Plugin.GetLoadedFrom() == EPluginLoadedFrom::Engine)
		{
			RaiseError(FString::Printf(TEXT("Cannot edit descriptor of engine plugin '%s' in an installed engine build."), *Plugin.GetName()));
			return false;
		}
		if (FApp::IsInstalled() && Plugin.GetType() != EPluginType::Mod)
		{
			RaiseError(FString::Printf(TEXT("Cannot edit descriptor of plugin '%s' in an installed build (only Mod plugins may be edited)."), *Plugin.GetName()));
			return false;
		}
		return true;
	}

	// Checks out the file via SCC if available, then calls IPlugin::UpdateDescriptor.
	// Skips the write entirely if the serialized descriptor text is unchanged.
	// Returns true on success; calls RaiseError and returns false on failure.
	bool UpdateDescriptorInternal(IPlugin& Plugin, const FPluginDescriptor& NewDescriptor)
	{
		if (!CanEditPluginDescriptor(Plugin))
		{
			return false;
		}

		FString OldText;
		Plugin.GetDescriptor().Write(OldText);
		FString NewText;
		NewDescriptor.Write(NewText);

		if (OldText.Compare(NewText, ESearchCase::CaseSensitive) == 0)
		{
			return true;
		}

		const FString DescriptorFileName = Plugin.GetDescriptorFileName();

		ISourceControlModule& SCCModule = ISourceControlModule::Get();
		if (SCCModule.IsEnabled())
		{
			ISourceControlProvider& Provider = SCCModule.GetProvider();
			TSharedPtr<ISourceControlState, ESPMode::ThreadSafe> State =
				Provider.GetState(DescriptorFileName, EStateCacheUsage::ForceUpdate);
			if (State.IsValid() && State->CanCheckout())
			{
				Provider.Execute(ISourceControlOperation::Create<FCheckOut>(), DescriptorFileName);
			}
		}

		FText FailReason;
		if (!Plugin.UpdateDescriptor(NewDescriptor, FailReason))
		{
			RaiseError(FailReason.ToString());
			return false;
		}
		return true;
	}

	FPluginTemplateDescriptionToolsetInfo ToToolsetTemplateDescription(FPluginTemplateDescription& PluginTemplate)
	{
		FPluginTemplateDescriptionToolsetInfo ToolsetInfo;
		ToolsetInfo.Name = PluginTemplate.Name;
		ToolsetInfo.Description = PluginTemplate.Description;
		ToolsetInfo.OnDiskPath = PluginTemplate.OnDiskPath;
		PluginTemplate.UpdatePluginNameTextWhenTemplateSelected(ToolsetInfo.DefaultTemplateName);
		ToolsetInfo.bCanBePlacedInEngine = PluginTemplate.bCanBePlacedInEngine && !FApp::IsInstalled();
		return ToolsetInfo;
	}

	bool ToolsetTemplateDescriptionMatches(const FPluginTemplateDescriptionToolsetInfo& ToolsetInfo, 
		const FPluginTemplateDescription& PluginTemplate)
	{
		return ToolsetInfo.Name.EqualTo(PluginTemplate.Name) && ToolsetInfo.Description.EqualTo(PluginTemplate.Description)
			&& ToolsetInfo.OnDiskPath == PluginTemplate.OnDiskPath;
	}
}


// --- Read Methods ---

TArray<FString> UPluginToolset::ListEnabledPlugins()
{
	TArray<FString> Names;
	for (const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetEnabledPlugins())
	{
		Names.Add(Plugin->GetName());
	}
	Names.Sort();
	return Names;
}

TArray<FString> UPluginToolset::ListDiscoveredPlugins()
{
	TArray<FString> Names;
	for (const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetDiscoveredPlugins())
	{
		Names.Add(Plugin->GetName());
	}
	Names.Sort();
	return Names;
}

FPluginToolsetInfo UPluginToolset::GetPluginInfo(const FString& PluginName)
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin.IsValid())
	{
		UE::PluginToolset::Private::RaiseError(FString::Printf(TEXT("Plugin could not be found: %s"), *PluginName));
		return FPluginToolsetInfo();
	}

	const FPluginDescriptor& Desc = Plugin->GetDescriptor();

	FPluginToolsetInfo Info;
	Info.Description = Desc.Description;
	Info.Version = Desc.Version;
	Info.VersionName = Desc.VersionName;
	Info.BaseDir = Plugin->GetBaseDir();
	Info.ContentDir = Plugin->GetContentDir();
	Info.DescriptorPath = Plugin->GetDescriptorFileName();
	Info.MountedAssetPath = Plugin->GetMountedAssetPath();
	return Info;
}

TArray<FPluginDependencyToolsetInfo> UPluginToolset::GetPluginDependencies(const FString& PluginName)
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin.IsValid())
	{
		UE::PluginToolset::Private::RaiseError(FString::Printf(TEXT("Plugin could not be found: %s"), *PluginName));
		return {};
	}

	TArray<FPluginDependencyToolsetInfo> Result;
	for (const FPluginReferenceDescriptor& Ref : Plugin->GetDescriptor().Plugins)
	{
		FPluginDependencyToolsetInfo& Entry = Result.AddDefaulted_GetRef();
		Entry.Name = Ref.Name;
		Entry.bOptional = Ref.bOptional;
		Entry.bEnabled = Ref.bEnabled;
	}
	return Result;
}

TArray<FString> UPluginToolset::GetPluginDependents(const FString& PluginName)
{
	if (!IPluginManager::Get().FindPlugin(PluginName).IsValid())
	{
		UE::PluginToolset::Private::RaiseError(FString::Printf(TEXT("Plugin could not be found: %s"), *PluginName));
		return {};
	}

	TArray<FString> Result;
	for (const TSharedRef<IPlugin>& Candidate : IPluginManager::Get().GetDiscoveredPlugins())
	{
		for (const FPluginReferenceDescriptor& Ref : Candidate->GetDescriptor().Plugins)
		{
			if (Ref.Name == PluginName)
			{
				Result.Add(Candidate->GetName());
				break;
			}
		}
	}
	Result.Sort();
	return Result;
}

bool UPluginToolset::IsEnabled(const FString& PluginName)
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin.IsValid())
	{
		UE::PluginToolset::Private::RaiseError(FString::Printf(TEXT("Plugin could not be found: %s"), *PluginName));
		return false;
	}
	return Plugin->IsEnabled();
}

FString UPluginToolset::GetPluginForAsset(const FString& AssetPath)
{
	for (const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetEnabledPluginsWithContent())
	{
		const FString MountedPath =	Plugin->GetMountedAssetPath();
		if (!MountedPath.IsEmpty() && AssetPath.StartsWith(MountedPath))
		{
			return Plugin->GetName();
		}
	}
	UE::PluginToolset::Private::RaiseError(FString::Printf(TEXT("No plugin contains asset: %s"),*AssetPath));
	return FString();
}

// --- Plugin Templates  ---

TArray<FPluginTemplateDescriptionToolsetInfo> UPluginToolset::GetPluginTemplateDescriptions()
{
	TArray<FPluginTemplateDescriptionToolsetInfo> ToolsetResult;
	if (IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
	{
		const IPluginsEditorFeature& PluginEditor = 
			IModularFeatures::Get().GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor);
		const TArray<TSharedRef<FPluginTemplateDescription>>& DefaultDescriptions = PluginEditor.GetDefaultPluginTemplates();
		const TArray<TSharedRef<FPluginTemplateDescription>>& AddedDescriptions = PluginEditor.GetAddedPluginTemplates();

		ToolsetResult.Reserve(DefaultDescriptions.Num() + AddedDescriptions.Num());

		for (const TSharedRef<FPluginTemplateDescription>& Description : DefaultDescriptions)
		{
			ToolsetResult.Emplace(UE::PluginToolset::Private::ToToolsetTemplateDescription(*Description));
		}
		for (const TSharedRef<FPluginTemplateDescription>& Description : AddedDescriptions)
		{
			ToolsetResult.Emplace(UE::PluginToolset::Private::ToToolsetTemplateDescription(*Description));
		}
	}

	return ToolsetResult;
}

TSharedPtr<FPluginTemplateDescription> UPluginToolset::FindPluginTemplateDescriptionForToolsetInfo(
	const FPluginTemplateDescriptionToolsetInfo& PluginTemplateInfo)
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
	{
		const IPluginsEditorFeature& PluginEditor = 
			IModularFeatures::Get().GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor);
		const TArray<TSharedRef<FPluginTemplateDescription>>& DefaultDescriptions = PluginEditor.GetDefaultPluginTemplates();
		for (const TSharedRef<FPluginTemplateDescription>& Description : DefaultDescriptions)
		{
			if (UE::PluginToolset::Private::ToolsetTemplateDescriptionMatches(PluginTemplateInfo, *Description))
			{
				return Description;
			}
		}
		const TArray<TSharedRef<FPluginTemplateDescription>>& AddedDescriptions = PluginEditor.GetAddedPluginTemplates();
		for (const TSharedRef<FPluginTemplateDescription>& Description : AddedDescriptions)
		{
			if (UE::PluginToolset::Private::ToolsetTemplateDescriptionMatches(PluginTemplateInfo, *Description))
			{
				return Description;
			}
		}
	}
	return TSharedPtr<FPluginTemplateDescription>();
}

// --- Plugin Creation ---

FString UPluginToolset::GeneratePluginFolderPath(FPluginTemplateDescription& PluginTemplate, bool bPlaceInEngine)
{
	FString PluginFolderPath = bPlaceInEngine ?
		IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::EnginePluginsDir()) :
		IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::ProjectPluginsDir());
	FPaths::MakePlatformFilename(PluginFolderPath);
	PluginTemplate.UpdatePathWhenTemplateSelected(PluginFolderPath);

	return PluginFolderPath;
}

bool UPluginToolset::ValidateNewPluginNameAndLocationInternal(
	const FString& PluginName,
	const FString& RelativePluginLocation,
	const bool bPlaceInEngine,
	FPluginTemplateDescription& PluginTemplate,
	FString& OutAbsolutePluginLocation)
{
	if (bPlaceInEngine && !PluginTemplate.bCanBePlacedInEngine)
	{
		UE::PluginToolset::Private::RaiseError(FString::Printf(TEXT("Plugin template %s cannot be placed in engine as requested."), *PluginTemplate.Name.ToString()));
		return false;
	}

	OutAbsolutePluginLocation = GeneratePluginFolderPath(PluginTemplate, bPlaceInEngine) / RelativePluginLocation;

	FText FailReason;
	if (!FPluginUtils::ValidateNewPluginNameAndLocation(PluginName, OutAbsolutePluginLocation, &FailReason))
	{
		UE::PluginToolset::Private::RaiseError(FailReason.ToString());
		return false;
	}
	const FString PluginFilePath = FPluginUtils::GetPluginFilePath(OutAbsolutePluginLocation, PluginName);

	if (!PluginTemplate.ValidatePathForPlugin(PluginFilePath, FailReason))
	{
		UE::PluginToolset::Private::RaiseError(FailReason.ToString());
		return false;
	}
	return true;
}

bool UPluginToolset::IsPluginCreationAllowed()
{
	bool bCreateEnabled = true;
	GConfig->GetBool(TEXT("EditorSettings"), TEXT("bCanCreatePluginsFromBrowser"), bCreateEnabled, GEditorIni);
	if (!bCreateEnabled)
	{
		UE::PluginToolset::Private::RaiseError(TEXT("Plugin creation is disabled in Editor Settings."));
		return false;
	}
	return true;
}

bool UPluginToolset::ValidateNewPluginNameAndLocation(
	const FString& PluginName,
	const FString& RelativePluginLocation,
	const bool bPlaceInEngine,
	const FPluginTemplateDescriptionToolsetInfo& TemplateInfo)
{
	TSharedPtr<FPluginTemplateDescription> Template = FindPluginTemplateDescriptionForToolsetInfo(TemplateInfo);
	if (!Template.IsValid())
	{
		UE::PluginToolset::Private::RaiseError(TEXT("Could not find a plugin template matching the provided TemplateInfo."));
		return false;
	}

	FString AbsolutePluginLocationUnused;
	return ValidateNewPluginNameAndLocationInternal(PluginName, RelativePluginLocation,
		bPlaceInEngine, *Template, AbsolutePluginLocationUnused);
}

FString UPluginToolset::CreatePlugin(
	const FString& PluginName,
	const FString& RelativePluginLocation,
	const bool bPlaceInEngine,
	const FPluginTemplateDescriptionToolsetInfo& TemplateInfo,
	const FString& Description)
{
	if (!IsPluginCreationAllowed())
	{
		return FString();
	}
	TSharedPtr<FPluginTemplateDescription> Template = FindPluginTemplateDescriptionForToolsetInfo(TemplateInfo);
	if (!Template.IsValid())
	{
		UE::PluginToolset::Private::RaiseError(TEXT("Could not find a plugin template matching the provided TemplateInfo."));
		return FString();
	}

	FString AbsolutePluginLocation;
	if (!ValidateNewPluginNameAndLocationInternal(PluginName, RelativePluginLocation,
		bPlaceInEngine, *Template, AbsolutePluginLocation))
	{
		return FString();
	}

	FPluginUtils::FNewPluginParamsWithDescriptor CreationParams;
	CreationParams.TemplateFolders.Add(Template->OnDiskPath);
	CreationParams.Descriptor.bCanContainContent = Template->bCanContainContent;
	CreationParams.Descriptor.FriendlyName = PluginName;
	CreationParams.Descriptor.Version = 1;
	CreationParams.Descriptor.VersionName = TEXT("1.0");
	CreationParams.Descriptor.Category = TEXT("Other");
	CreationParams.Descriptor.Description = Description;
	CreationParams.PostCreatePythonScriptPath = Template->PostCreatePythonScriptPath;
	CreationParams.PostCreatePythonScriptArguments = Template->PostCreatePythonScriptArguments;

	Template->CustomizeDescriptorBeforeCreation(CreationParams.Descriptor);

	FText FailReason;
	FPluginUtils::FLoadPluginParams LoadParams;
	LoadParams.bEnablePluginInProject = true;
	LoadParams.bUpdateProjectPluginSearchPath = true;
	LoadParams.OutFailReason = &FailReason;

	TSharedPtr<IPlugin> NewPlugin = FPluginUtils::CreateAndLoadNewPlugin(
		PluginName, AbsolutePluginLocation, CreationParams, LoadParams);
	if (!NewPlugin.IsValid())
	{
		UE::PluginToolset::Private::RaiseError(FailReason.ToString());
		return FString();
	}

	Template->OnPluginCreated(NewPlugin);
	return NewPlugin->GetDescriptorFileName();
}

// --- Plugin Management ---

bool UPluginToolset::IsPluginModificationAllowed()
{
	bool bValue = true;
	GConfig->GetBool(TEXT("EditorSettings"), TEXT("bCanModifyPluginsFromBrowser"), bValue, GEditorIni);
	if (!bValue)
	{
		UE::PluginToolset::Private::RaiseError(TEXT("Plugin modification is disabled in Editor Settings."));
		return false;
	}
	return true;
}

void UPluginToolset::SetPluginEnabled(const FString& PluginName, bool bEnabled)
{
	if (!IsPluginModificationAllowed())
	{
		return;
	}
	if (!IPluginManager::Get().FindPlugin(PluginName).IsValid())
	{
		UE::PluginToolset::Private::RaiseError(FString::Printf(TEXT("Plugin could not be found: %s"), *PluginName));
		return;
	}
	FText FailReason;
	if (!IProjectManager::Get().SetPluginEnabled(PluginName, bEnabled, FailReason))
	{
		UE::PluginToolset::Private::RaiseError(FailReason.ToString());
	}
}

// --- Plugin Descriptor Editing ---

FPluginDescriptorToolsetInfo UPluginToolset::GetPluginDescriptor(const FString& PluginName)
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin.IsValid())
	{
		UE::PluginToolset::Private::RaiseError(FString::Printf(TEXT("Plugin could not be found: %s"), *PluginName));
		return FPluginDescriptorToolsetInfo();
	}

	const FPluginDescriptor& Desc = Plugin->GetDescriptor();

	FPluginDescriptorToolsetInfo Info;
	Info.FriendlyName = Desc.FriendlyName;
	Info.Description = Desc.Description;
	Info.Category = Desc.Category;
	Info.VersionName = Desc.VersionName;
	Info.Version = Desc.Version;
	Info.CreatedBy = Desc.CreatedBy;
	Info.CreatedByURL = Desc.CreatedByURL;
	Info.DocsURL = Desc.DocsURL;
	Info.MarketplaceURL = Desc.MarketplaceURL;
	Info.SupportURL = Desc.SupportURL;
	Info.bCanContainContent = Desc.bCanContainContent;
	Info.bIsBetaVersion = Desc.bIsBetaVersion;
	Info.bIsExperimentalVersion = Desc.bIsExperimentalVersion;
	Info.bIsSealed = Desc.bIsSealed;
	return Info;
}

void UPluginToolset::UpdatePluginDescriptor(const FString& PluginName, const FPluginDescriptorToolsetInfo& NewDescriptor)
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin.IsValid())
	{
		UE::PluginToolset::Private::RaiseError(FString::Printf(TEXT("Plugin could not be found: %s"), *PluginName));
		return;
	}

	FPluginDescriptor UpdatedDescriptor = Plugin->GetDescriptor();
	UpdatedDescriptor.FriendlyName = NewDescriptor.FriendlyName;
	UpdatedDescriptor.Description = NewDescriptor.Description;
	UpdatedDescriptor.Category = NewDescriptor.Category;
	UpdatedDescriptor.VersionName = NewDescriptor.VersionName;
	UpdatedDescriptor.Version = NewDescriptor.Version;
	UpdatedDescriptor.CreatedBy = NewDescriptor.CreatedBy;
	UpdatedDescriptor.CreatedByURL = NewDescriptor.CreatedByURL;
	UpdatedDescriptor.DocsURL = NewDescriptor.DocsURL;
	UpdatedDescriptor.MarketplaceURL = NewDescriptor.MarketplaceURL;
	UpdatedDescriptor.SupportURL = NewDescriptor.SupportURL;
	UpdatedDescriptor.bCanContainContent = NewDescriptor.bCanContainContent;
	UpdatedDescriptor.bIsBetaVersion = NewDescriptor.bIsBetaVersion;
	UpdatedDescriptor.bIsExperimentalVersion = NewDescriptor.bIsExperimentalVersion;
	UpdatedDescriptor.bIsSealed = NewDescriptor.bIsSealed;

	UE::PluginToolset::Private::UpdateDescriptorInternal(*Plugin, UpdatedDescriptor);
}

void UPluginToolset::AddPluginDependency(const FString& PluginName, const FString& DependencyName, bool bOptional, bool bEnabled)
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin.IsValid())
	{
		UE::PluginToolset::Private::RaiseError(FString::Printf(TEXT("Plugin could not be found: %s"), *PluginName));
		return;
	}

	FPluginDescriptor UpdatedDescriptor = Plugin->GetDescriptor();

	for (const FPluginReferenceDescriptor& Existing : UpdatedDescriptor.Plugins)
	{
		if (Existing.Name == DependencyName)
		{
			if (Existing.bOptional == bOptional && Existing.bEnabled == bEnabled)
			{
				return;
			}
			UE::PluginToolset::Private::RaiseError(FString::Printf(
				TEXT("Dependency '%s' already exists with different settings in plugin '%s'."),
				*DependencyName, *PluginName));
			return;
		}
	}

	FPluginReferenceDescriptor& NewDep = UpdatedDescriptor.Plugins.AddDefaulted_GetRef();
	NewDep.Name = DependencyName;
	NewDep.bOptional = bOptional;
	NewDep.bEnabled = bEnabled;

	UE::PluginToolset::Private::UpdateDescriptorInternal(*Plugin, UpdatedDescriptor);
}

void UPluginToolset::RemovePluginDependency(const FString& PluginName, const FString& DependencyName)
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin.IsValid())
	{
		UE::PluginToolset::Private::RaiseError(FString::Printf(TEXT("Plugin could not be found: %s"), *PluginName));
		return;
	}

	FPluginDescriptor UpdatedDescriptor = Plugin->GetDescriptor();

	const int32 RemovedCount = UpdatedDescriptor.Plugins.RemoveAll(
		[&DependencyName](const FPluginReferenceDescriptor& Ref) { return Ref.Name == DependencyName; });

	if (RemovedCount == 0)
	{
		UE::PluginToolset::Private::RaiseError(FString::Printf(
			TEXT("Dependency '%s' not found in plugin '%s'."), *DependencyName, *PluginName));
		return;
	}

	UE::PluginToolset::Private::UpdateDescriptorInternal(*Plugin, UpdatedDescriptor);
}
