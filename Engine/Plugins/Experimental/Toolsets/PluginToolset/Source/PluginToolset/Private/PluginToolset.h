// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PluginUtils.h"
#include "ToolsetRegistry/ToolsetDefinition.h"

#include "PluginToolset.generated.h"

struct FPluginTemplateDescription;

/// Editable metadata fields from a plugin's descriptor.
USTRUCT(BlueprintType)
struct FPluginDescriptorToolsetInfo
{
	GENERATED_BODY()

	/** Friendly name of the plugin. */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	FString FriendlyName;

	/** Description of the plugin. */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	FString Description;

	/** The name of the category this plugin belongs to. */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	FString Category;

	/**
	 * Name of the version for this plugin. This is the front-facing part of the version number. It doesn't need to
	 * match the version number numerically, but should be updated when the version number is increased accordingly.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	FString VersionName;

	/**
	 * Version number for the plugin. The version number must increase with every version of the plugin, so that the
	 * system can determine whether one version of a plugin is newer than another, or to enforce other requirements.
	 * This version number is not displayed in front-facing UI. Use the VersionName for that.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	int32 Version = 0;

	/** The company or individual who created this plugin. This is an optional field that may be displayed in the user interface. */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	FString CreatedBy;

	/** Hyperlink URL string for the company or individual who created this plugin. This is optional. */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	FString CreatedByURL;

	/** Documentation URL string. */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	FString DocsURL;

	/** Marketplace URL for this plugin. This URL will be embedded into projects that enable this plugin, so we can redirect to the marketplace if a user doesn't have it installed. */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	FString MarketplaceURL;

	/** Support URL/email for this plugin. */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	FString SupportURL;

	/** Can this plugin contain content? */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	bool bCanContainContent = false;

	/** Marks the plugin as beta in the UI. */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	bool bIsBetaVersion = false;

	/** Marks the plugin as experimental in the UI. */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	bool bIsExperimentalVersion = false;

	/** Prevents other plugins from depending on this plugin. */
	UPROPERTY(BlueprintReadWrite, Category = "PluginToolset")
	bool bIsSealed = false;
};

/// Metadata about a plugin.
USTRUCT(BlueprintType)
struct FPluginToolsetInfo
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	FString Description;

	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	int32 Version = 0;

	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	FString VersionName;

	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	FString BaseDir;

	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	FString ContentDir;

	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	FString DescriptorPath;

	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	FString MountedAssetPath;
};

/// A single dependency entry from a plugin's Plugins array.
USTRUCT(BlueprintType)
struct FPluginDependencyToolsetInfo
{
	GENERATED_BODY()

	/** Name of the dependency plugin. */
	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	FString Name;

	/** Whether the dependency is optional. */
	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	bool bOptional = false;

	/** Whether the dependency should be enabled. */
	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	bool bEnabled = true;
};

/// Metadata about a plugin template description.
USTRUCT(BlueprintType)
struct FPluginTemplateDescriptionToolsetInfo
{
	GENERATED_BODY()

	/** Name of this template in the GUI */
	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	FText Name;

	/** Description of this template in the GUI */
	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	FText Description;

	/** Path to the directory containing template files. */
	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	FString OnDiskPath;

	/** Default plugin name. Typically a prefix. */
	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	FText DefaultTemplateName;

	/** Whether or not this template can be used for Engine plugins */
	UPROPERTY(BlueprintReadOnly, Category = "PluginToolset")
	bool bCanBePlacedInEngine = false;
};

/// Tools for creating, editing, enabling, and querying Unreal plugins
UCLASS(BlueprintType, MinimalAPI)
class UPluginToolset : public UToolsetDefinition
{
	GENERATED_BODY()
public:

	// --- Read Methods ---

	/**
	 * Lists the names of all enabled plugins, sorted alphabetically.
	 * @return Sorted array of enabled plugin names.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static TArray<FString> ListEnabledPlugins();

	/**
	 * Lists the names of all discovered plugins (enabled and disabled), sorted alphabetically.
	 * @return Sorted array of discovered plugin names.
	 */
	UFUNCTION(meta = (AICallable), Category = "PluginToolset")
	static TArray<FString> ListDiscoveredPlugins();

	/**
	 * Gets metadata for a discovered plugin, including description, version, base directory,
	 * content directory, descriptor path, and mounted asset path.
	 * @param PluginName The name of the plugin.
	 * @return Metadata for the plugin.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static FPluginToolsetInfo GetPluginInfo(const FString& PluginName);

	/**
	 * Checks whether a discovered plugin is currently enabled.
	 * @param PluginName The name of the plugin.
	 * @return True if the plugin is enabled, false if it is disabled.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static bool IsEnabled(const FString& PluginName);

	/**
	 * Returns the dependency entries from a plugin's Plugins array in its .uplugin file.
	 * @param PluginName The name of the plugin.
	 * @return Array of dependency entries declared in the plugin's .uplugin file.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static TArray<FPluginDependencyToolsetInfo> GetPluginDependencies(const FString& PluginName);

	/**
	 * Returns the names of all discovered plugins that declare a dependency on the given plugin.
	 * @param PluginName The name of the plugin to search for as a dependency target.
	 * @return Sorted array of plugin names that list the given plugin in their Plugins array.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static TArray<FString> GetPluginDependents(const FString& PluginName);

	/**
	 * Returns the name of the enabled plugin whose content mount point contains the given asset path.
	 * Accepts full asset paths or mount point prefixes (e.g. /PluginName/ or /Game/Path/To/Asset).
	 * @param AssetPath The asset or mount point path to look up.
	 * @return The name of the plugin that owns the asset.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static FString GetPluginForAsset(const FString& AssetPath);

	// --- Plugin Templates  ---
	/**
	 * Returns the list of available plugin templates. Pass one of the results to CreatePlugin
	 * to create a new plugin from that template.
	 * @return Array of available plugin template descriptors.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static TArray<FPluginTemplateDescriptionToolsetInfo> GetPluginTemplateDescriptions();

	// --- Plugin Creation ---

	/**
	 * Checks whether the editor settings permit plugin creation from the plugin browser.
	 * @return True if plugin creation is allowed, false if it is disabled in Editor Settings.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static bool IsPluginCreationAllowed();

	/**
	 * Validates that PluginName and RelativePluginLocation are acceptable for a new plugin.
	 * @param PluginName The proposed plugin name.
	 * @param RelativePluginLocation Parent directory for the new plugin relative to template's default location.
	 *     This should be empty unless you wish to specify a subdirectory.
	 * @param bPlaceInEngine Use Engine Plugins directory rather than Game Plugins directory location.
	 *     Only some Templates allow placing in Engine. See the TemplateInfo's bCanBePlacedInEngine.
	 *     This should be false unless explicitly requested by the user.
	 * @param TemplateInfo The plugin template to potentially create from.
	 * @return True if the name and location are valid.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static bool ValidateNewPluginNameAndLocation(
		const FString& PluginName,
		const FString& RelativePluginLocation,
		const bool bPlaceInEngine,
		const FPluginTemplateDescriptionToolsetInfo& TemplateInfo);

	/**
	 * Creates a new plugin from a template and loads it into the editor.
	 * Use GetPluginTemplateDescriptions to obtain a valid TemplateInfo.
	 * @param PluginName Name for the new plugin.
	 * @param RelativePluginLocation Parent directory for the new plugin relative to template's default location.
	 *     This should be empty unless you wish to specify a subdirectory.
	 * @param bPlaceInEngine Use Engine Plugins directory rather than Game Plugins directory location.
	 *     Only some Templates allow placing in Engine. See the TemplateInfo's bCanBePlacedInEngine.
	 *     This should be false unless explicitly requested by the user.
	 * @param TemplateInfo The plugin template to create from.
	 * @param Description A description for the new plugin.
	 * @return Created plugin's descriptor filename. Empty on failure.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static FString CreatePlugin(
		const FString& PluginName,
		const FString& RelativePluginLocation,
		const bool bPlaceInEngine,
		const FPluginTemplateDescriptionToolsetInfo& TemplateInfo,
		const FString& Description);

	// --- Plugin Management ---

	/**
	 * Checks whether the editor settings permit modifying plugins from the plugin browser.
	 * @return True if plugin modification is allowed, false if it is disabled in Editor Settings.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static bool IsPluginModificationAllowed();

	/**
	 * Enables or disables a plugin in the project config. The change takes effect on the next editor restart.
	 * @param PluginName The name of the plugin.
	 * @param bEnabled True to enable, false to disable.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static void SetPluginEnabled(const FString& PluginName, bool bEnabled);

	// --- Plugin Descriptor Editing ---

	/**
	 * Gets the editable descriptor fields for a discovered plugin.
	 * @param PluginName The name of the plugin.
	 * @return The plugin's current editable descriptor fields.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static FPluginDescriptorToolsetInfo GetPluginDescriptor(const FString& PluginName);

	/**
	 * Updates a plugin's descriptor fields and writes them to its .uplugin file.
	 * Checks out the file via source control if source control is enabled.
	 * No-ops if the serialized descriptor is unchanged (file is not touched).
	 * @param PluginName The name of the plugin to update.
	 * @param NewDescriptor The new descriptor field values to apply.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static void UpdatePluginDescriptor(const FString& PluginName, const FPluginDescriptorToolsetInfo& NewDescriptor);

	/**
	 * Adds a dependency entry to a plugin's Plugins array in its .uplugin file.
	 * No-ops if a dependency with that name already exists with matching settings.
	 * The dependency plugin does not need to be currently discovered.
	 * @param PluginName The name of the plugin to modify.
	 * @param DependencyName The name of the plugin to add as a dependency.
	 * @param bOptional Whether the dependency is optional.
	 * @param bEnabled Whether the dependency should be enabled.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static void AddPluginDependency(const FString& PluginName, const FString& DependencyName, bool bOptional, bool bEnabled);

	/**
	 * Removes a dependency entry from a plugin's Plugins array in its .uplugin file.
	 * @param PluginName The name of the plugin to modify.
	 * @param DependencyName The name of the dependency to remove.
	 */
	UFUNCTION(meta=(AICallable), Category = "PluginToolset")
	static void RemovePluginDependency(const FString& PluginName, const FString& DependencyName);

private:
	friend class FPluginToolsetSpec;

	/**
	 * Resolves a FPluginTemplateDescriptionToolsetInfo to its underlying FPluginTemplateDescription.
	 * @param PluginTemplateInfo The toolset info to look up.
	 * @return The matching template description, or nullptr if no match was found.
	 */
	static TSharedPtr<FPluginTemplateDescription> FindPluginTemplateDescriptionForToolsetInfo(
		const FPluginTemplateDescriptionToolsetInfo& PluginTemplateInfo);

	static FString GeneratePluginFolderPath(FPluginTemplateDescription& PluginTemplate, bool bPlaceInEngine);

	static bool ValidateNewPluginNameAndLocationInternal(const FString& PluginName, const FString& RelativePluginLocation,
		const bool bPlaceInEngine, FPluginTemplateDescription& PluginTemplate, FString& OutAbsolutePluginLocation);
};
