// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/UToolsetRegistry.h"
#include "PluginToolset.h"

class FPluginToolsetModule : public IModuleInterface
{
	void ShutdownModule()
	{
		// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
		// we call this function before unloading the module.
		UToolsetRegistry::UnregisterToolsetClass(UPluginToolset::StaticClass());
	}

	void StartupModule()
	{
		// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
		UToolsetRegistry::RegisterToolsetClass(UPluginToolset::StaticClass());
	}

};
	
IMPLEMENT_MODULE(FPluginToolsetModule, PluginToolset)