// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Features/EditorFeatures.h"
#include "Features/IPluginsEditorFeature.h"
#include "Features/IModularFeatures.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"
#include "HAL/FileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "PluginReferenceDescriptor.h"
#include "SourceControlHelpers.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"
#include "PluginUtils.h"

#include "PluginToolset.h"

#if WITH_DEV_AUTOMATION_TESTS
namespace
{
const EAutomationTestFlags TestFlags =
	EAutomationTestFlags::EditorContext |
	EAutomationTestFlags::ProductFilter |
	EAutomationTestFlags::CriticalPriority;


// Test plugin template (mostly exists so it can be marked as explicitly loaded)
struct FPluginToolsetTestPluginTemplate : public FPluginTemplateDescription
{
	FPluginToolsetTestPluginTemplate()
		: FPluginTemplateDescription(FText::FromString(TEXT("PluginToolsetTestPluginTemplate")),
			FText::FromString(TEXT("Plugin template for testing creating plugins")),
			IPluginManager::Get().FindPlugin(TEXT("PluginToolset"))->GetBaseDir() / TEXT("TestPluginTemplate"),
			false, EHostType::Editor)
	{
	}

	virtual void CustomizeDescriptorBeforeCreation(FPluginDescriptor& Descriptor) override
	{
		Descriptor.bExplicitlyLoaded = true;
	}
};
}

BEGIN_DEFINE_SPEC(
	FPluginToolsetSpec,
	TEXT("AI.Toolsets.PluginToolset"),
	TestFlags)

// Ensure no blueprint script exceptions were caught and clear the handler.
void ExpectNoException()
{
	check(ExceptionHandler.IsValid());
	TestEqual(TEXT("Error"), *ExceptionHandler->GetException(), TEXT(""));
	ExceptionHandler.Reset();
}

// Ensure blueprint script exception was caught and contains the error message, then clear the
// handler.
void ExpectExceptionContains(const FString& ErrorMessage)
{
	check(ExceptionHandler.IsValid());
	TestTrue(*FString::Printf(TEXT("Error contains '%s'"), *ErrorMessage),
		ExceptionHandler->GetException().Contains(ErrorMessage));
	ExceptionHandler.Reset();
}

// Ensure blueprint script exception was caught and clear the handler.
void ExpectException()
{
	check(ExceptionHandler.IsValid());
	TestNotEqual(TEXT("Has Error"), *ExceptionHandler->GetException(), TEXT(""));
	ExceptionHandler.Reset();
}

// Add a test template so at least one template exists
void AddTestTemplateDescription()
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
	{
		if (!TestTemplate.IsValid())
		{
			TestTemplate = MakeShareable(new FPluginToolsetTestPluginTemplate());
		}
		IPluginsEditorFeature& PluginEditor =
			IModularFeatures::Get().GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor);
		PluginEditor.RegisterPluginTemplate(TestTemplate.ToSharedRef());
	}
}

void RemoveTestTemplateDescription()
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
	{
		if (ensure(TestTemplate.IsValid()))
		{
			IPluginsEditorFeature& PluginEditor =
				IModularFeatures::Get().GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor);
				PluginEditor.UnregisterPluginTemplate(TestTemplate.ToSharedRef());
		}
	}
}

private:
	TUniquePtr<UE::ToolsetRegistry::FToolCallExceptionHandler> ExceptionHandler;
	TSharedPtr<FPluginTemplateDescription> TestTemplate;

END_DEFINE_SPEC(FPluginToolsetSpec)

void FPluginToolsetSpec::Define()
{
	BeforeEach([this]()
	{
		ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
	});

	AfterEach([this]()
	{
		if (ExceptionHandler.IsValid())
		{
			ExpectNoException();
		}
	});


	Describe(TEXT("ListEnabledPlugins"), [this]()
	{
		It(TEXT("returns a sorted list"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UPluginToolset::ListEnabledPlugins();
				for (int32 i = 1; i < Names.Num(); ++i)
				{
					TestTrue(TEXT("Sorted"), Names[i - 1] <= Names[i]);
				}
			});
			ExpectNoException();
		});

		It(TEXT("Contains ToolsetRegistry"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UPluginToolset::ListEnabledPlugins();
				TestTrue(TEXT("Contains ToolsetRegistry"), Names.Contains(TEXT("ToolsetRegistry")));
			});
			ExpectNoException();
		});

		It(TEXT("all listed plugins are enabled according to the plugin manager"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UPluginToolset::ListEnabledPlugins();
				for (const FString& Name : Names)
				{
					TestTrue(TEXT("Found plugin with name"), IPluginManager::Get().FindEnabledPlugin(Name).IsValid());
				}
			});
			ExpectNoException();
		});
	});

	Describe(TEXT("ListDiscoveredPlugins"), [this]()
	{
		It(TEXT("returns a sorted list"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UPluginToolset::ListDiscoveredPlugins();
				for (int32 i = 1; i < Names.Num(); ++i)
				{
					TestTrue(TEXT("Sorted"), Names[i - 1] <= Names[i]);
				}
			});
			ExpectNoException();
		});

		It(TEXT("Contains ToolsetRegistry"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UPluginToolset::ListDiscoveredPlugins();
				TestTrue(TEXT("Contains ToolsetRegistry"), Names.Contains(TEXT("ToolsetRegistry")));
			});
			ExpectNoException();
		});

		It(TEXT("all listed plugins are discovered according to the plugin manager"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UPluginToolset::ListDiscoveredPlugins();
				for (const FString& Name : Names)
				{
					TestTrue(TEXT("Found plugin with name"), IPluginManager::Get().FindPlugin(Name).IsValid());
				}
			});
			ExpectNoException();
		});
	});

	Describe(TEXT("GetPluginInfo"), [this]()
	{
		It(TEXT("Returns metadata for enabled plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const FPluginToolsetInfo Info = UPluginToolset::GetPluginInfo(TEXT("ToolsetRegistry"));
				TestFalse(TEXT("Has base dir"), Info.BaseDir.IsEmpty());
			});
			ExpectNoException();
		});

		It(TEXT("Works for code-only plugins"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const FPluginToolsetInfo Info = UPluginToolset::GetPluginInfo(TEXT("PythonScriptPlugin"));
				TestFalse(TEXT("Has base dir"), Info.BaseDir.IsEmpty());
			});
			ExpectNoException();
		});

		It(TEXT("Raises error for nonexistent plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const FPluginToolsetInfo Info = UPluginToolset::GetPluginInfo(TEXT("NonexistentXYZ12345"));
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});
	});

	Describe(TEXT("IsEnabled"), [this]()
	{
		It(TEXT("Returns true for ToolsetRegistry"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				TestTrue(TEXT("Enabled"), UPluginToolset::IsEnabled(TEXT("ToolsetRegistry")));
			});
			ExpectNoException();
		});

		It(TEXT("matches plugin's enabled status"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				for (const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetDiscoveredPlugins())
				{
					TestEqual(TEXT("Enabled"), Plugin->IsEnabled(), UPluginToolset::IsEnabled(Plugin->GetName()));
				}
			});
			ExpectNoException();
		});

		It(TEXT("raise an exception for non-existent plugins"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::IsEnabled(TEXT("NonexistentPluginXYZ12345"));
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});
	});

	Describe(TEXT("GetPluginForAsset"), [this]()
	{
		It(TEXT("Returns plugin name for known asset"),
			[this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				FString Result = UPluginToolset::GetPluginForAsset(TEXT("/ToolsetRegistry/"));
				TestEqual(TEXT("Plugin name"), Result, FString(TEXT("ToolsetRegistry")));
			});
			ExpectNoException();
		});

		It(TEXT("Raises error for unknown asset"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				FString Result = UPluginToolset::GetPluginForAsset(TEXT("/Game/NonexistentAsset12345"));
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});
	});



	Describe(TEXT("PluginTemplateDescriptions"), [this]()
	{
		It(TEXT("Returns a list of plugin template descriptions"),
			[this]()
		{
			if (!IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				AddInfo(TEXT("PluginsEditor modular feature not available. Skipping test."));
				return;
			}
			// Ensure at least one template description is available.
			AddTestTemplateDescription();
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				TArray<FPluginTemplateDescriptionToolsetInfo> Result = UPluginToolset::GetPluginTemplateDescriptions();
				TestTrue(TEXT("Non-empty result"), !Result.IsEmpty());
			});
			ExpectNoException();
			RemoveTestTemplateDescription();
		});

		It(TEXT("all results can be found by FindPluginTemplateDescriptionForToolsetInfo"), [this]()
		{
			if (!IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				AddInfo(TEXT("PluginsEditor modular feature not available. Skipping test."));
				return;
			}
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FPluginTemplateDescriptionToolsetInfo> Infos = UPluginToolset::GetPluginTemplateDescriptions();
				for (const FPluginTemplateDescriptionToolsetInfo& Info : Infos)
				{
					TestTrue(TEXT("Found template"), UPluginToolset::FindPluginTemplateDescriptionForToolsetInfo(Info).IsValid());
				}
			});
			ExpectNoException();
		});

		It(TEXT("empty info produces null result from FindPluginTemplateDescriptionForToolsetInfo"), [this]()
		{
			if (!IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				AddInfo(TEXT("PluginsEditor modular feature not available. Skipping test."));
				return;
			}
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				TestFalse(TEXT("Null result"),
					UPluginToolset::FindPluginTemplateDescriptionForToolsetInfo(FPluginTemplateDescriptionToolsetInfo{}).IsValid());
			});
			ExpectNoException();
		});
	});

	Describe(TEXT("IsPluginCreationAllowed"), [this]()
	{
		It(TEXT("matches GEditorIni bCanCreatePluginsFromBrowser"), [this]()
		{
			bool bExpected = true;
			GConfig->GetBool(TEXT("EditorSettings"), TEXT("bCanCreatePluginsFromBrowser"), bExpected, GEditorIni);

			bool bResult = false;
			ExceptionHandler->CaptureErrorsIn([this, &bResult]()
			{
				bResult = UPluginToolset::IsPluginCreationAllowed();
			});

			TestEqual(TEXT("Result matches config"), bResult, bExpected);
			if (bExpected)
			{
				ExpectNoException();
			}
			else
			{
				ExpectExceptionContains(TEXT("PluginToolset"));
			}
		});
	});

	Describe(TEXT("ValidateNewPluginNameAndLocation"), [this]()
	{
		It(TEXT("Raises error for unresolvable template"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				TestFalse(TEXT("Valid"), UPluginToolset::ValidateNewPluginNameAndLocation(
					TEXT("TestPluginXYZ12345Valid"), TEXT(""), false, FPluginTemplateDescriptionToolsetInfo{}));
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});

		It(TEXT("Returns true for valid name and location"), [this]()
		{
			if (!IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				AddInfo(TEXT("PluginsEditor modular feature not available. Skipping test."));
				return;
			}
			// Ensure at least one template description is available.
			AddTestTemplateDescription();
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FPluginTemplateDescriptionToolsetInfo> Infos = UPluginToolset::GetPluginTemplateDescriptions();
				if (TestTrue(TEXT("Non-empty template descriptions."), !Infos.IsEmpty()))
				{
					TestTrue(TEXT("Valid"), UPluginToolset::ValidateNewPluginNameAndLocation(
						TEXT("TestPluginXYZ12345Valid"), TEXT(""), false, Infos[0]));
				}
			});
			ExpectNoException();
			RemoveTestTemplateDescription();
		});

		It(TEXT("Raises error for taken plugin name"), [this]()
		{
			if (!IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				AddInfo(TEXT("PluginsEditor modular feature not available. Skipping test."));
				return;
			}

			// Ensure at least one template description is available.
			AddTestTemplateDescription();
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FPluginTemplateDescriptionToolsetInfo> Infos = UPluginToolset::GetPluginTemplateDescriptions();
				if (TestTrue(TEXT("Non-empty template descriptions."), !Infos.IsEmpty()))
				{
					TestFalse(TEXT("Valid"), UPluginToolset::ValidateNewPluginNameAndLocation(
						TEXT("ToolsetRegistry"), TEXT(""), false, Infos[0]));
				}
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
			RemoveTestTemplateDescription();
		});

		It(TEXT("Raises error for empty name"), [this]()
		{
			if (!IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				AddInfo(TEXT("PluginsEditor modular feature not available. Skipping test."));
				return;
			}

			// Ensure at least one template description is available.
			AddTestTemplateDescription();
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FPluginTemplateDescriptionToolsetInfo> Infos = UPluginToolset::GetPluginTemplateDescriptions();
				if (TestTrue(TEXT("Non-empty template descriptions."), !Infos.IsEmpty()))
				{
					TestFalse(TEXT("Valid"), UPluginToolset::ValidateNewPluginNameAndLocation(
						TEXT(""), TEXT(""), false, Infos[0]));
				}
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
			RemoveTestTemplateDescription();
		});
	});

	Describe(TEXT("CreatePlugin"), [this]()
	{
		It(TEXT("Raises error for taken plugin name"), [this]()
		{
			if (!IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				AddInfo(TEXT("PluginsEditor modular feature not available. Skipping test."));
				return;
			}

			// Ensure at least one template description is available.
			AddTestTemplateDescription();
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FPluginTemplateDescriptionToolsetInfo> Infos = UPluginToolset::GetPluginTemplateDescriptions();
				if (TestTrue(TEXT("Non-empty template descriptions."), !Infos.IsEmpty()))
				{
					const FString Result = UPluginToolset::CreatePlugin(
						TEXT("ToolsetRegistry"), TEXT(""), false, Infos[0], TEXT(""));
					TestTrue(TEXT("Empty result"), Result.IsEmpty());
				}
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
			RemoveTestTemplateDescription();
		});

		It(TEXT("Raises error for unresolvable template"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const FString Result = UPluginToolset::CreatePlugin(
					TEXT("TestPluginXYZ12345Valid"), TEXT(""), false,
					FPluginTemplateDescriptionToolsetInfo{}, TEXT(""));
				TestTrue(TEXT("Empty result"), Result.IsEmpty());
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});

		It(TEXT("Successfully creates a plugin"), [this]()
		{
			bool bCreationAllowed = true;
			GConfig->GetBool(TEXT("EditorSettings"), TEXT("bCanCreatePluginsFromBrowser"), bCreationAllowed, GEditorIni);
			if (!bCreationAllowed)
			{
				AddInfo(TEXT("Plugin creation is not allowed in this project. Skipping test."));
				ExceptionHandler.Reset();
				return;
			}
			if (!IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				AddInfo(TEXT("PluginsEditor modular feature not available. Skipping test."));
				return;
			}

			// Add test template to plugin editor
			AddTestTemplateDescription();

			// Create our info for our actual test.
			FPluginTemplateDescriptionToolsetInfo Info;
			Info.Name = TestTemplate->Name;
			Info.Description = TestTemplate->Description;
			Info.OnDiskPath = TestTemplate->OnDiskPath;
			Info.bCanBePlacedInEngine = false;
			Info.DefaultTemplateName = FText();

			const FString PluginName = TEXT("PluginToolsetTestXYZ12345");
			const FString PluginRelativeLocation = TEXT("Nested");
			const FString ExpectedPluginAbsoluteLocation = FPaths::ProjectPluginsDir() / PluginRelativeLocation;
			const FString ExpectedPluginPath = FPluginUtils::GetPluginFilePath(ExpectedPluginAbsoluteLocation, PluginName, /*bFullPath*/ true);
			FString PluginAbsoluteLocation, PluginPath;
			ExceptionHandler->CaptureErrorsIn([this, Info, &PluginName, &PluginRelativeLocation, &ExpectedPluginPath, &ExpectedPluginAbsoluteLocation, &PluginAbsoluteLocation, &PluginPath]()
			{
				PluginAbsoluteLocation = UPluginToolset::GeneratePluginFolderPath(*TestTemplate, false) / PluginRelativeLocation;
				TestTrue(TEXT("Result plugin folder path matches"), FPaths::IsSamePath(ExpectedPluginAbsoluteLocation, PluginAbsoluteLocation));

				PluginPath = UPluginToolset::CreatePlugin(PluginName, PluginRelativeLocation, false, Info, TEXT("Test plugin"));
				TestTrue(TEXT("Result path matches"),
					FPaths::IsSamePath(ExpectedPluginPath, PluginPath));
			});


			// Now clean up
			// Remove from project config before unloading so that RefreshPluginsList (called
			// internally by UnloadPlugin) doesn't re-enable the plugin when it re-discovers it.
			FText DisableReason;
			IProjectManager::Get().SetPluginEnabled(PluginName, false, DisableReason);

			FText UnloadFailReason;
			FPluginUtils::UnloadPlugin(PluginName, &UnloadFailReason);

			// RemoveFromPluginsList expects the .uplugin file path, not the plugin name.
			FText RemoveFailReason;
			TestTrue(TEXT("PluginRemoveSuccess"), IPluginManager::Get().RemoveFromPluginsList(PluginPath, &RemoveFailReason));

			if (USourceControlHelpers::IsAvailable())
			{
				// Revert the pending p4 add operations for all files in the plugin directory.
				TArray<FString> FilesToRevert;
				IFileManager::Get().FindFilesRecursive(FilesToRevert, *(PluginAbsoluteLocation / PluginName), TEXT("*"), true, false);
				if (FilesToRevert.Num() > 0)
				{
					USourceControlHelpers::RevertFiles(FilesToRevert);
				}
			}
			IFileManager::Get().DeleteDirectory(*(PluginAbsoluteLocation / PluginName), false, true);

			RemoveTestTemplateDescription();
			ExpectNoException();
		});
	});

	Describe(TEXT("IsPluginModificationAllowed"), [this]()
	{
		It(TEXT("matches GEditorIni bCanModifyPluginsFromBrowser"), [this]()
		{
			bool bExpected = true;
			GConfig->GetBool(TEXT("EditorSettings"), TEXT("bCanModifyPluginsFromBrowser"), bExpected, GEditorIni);

			bool bResult = false;
			ExceptionHandler->CaptureErrorsIn([this, &bResult]()
			{
				bResult = UPluginToolset::IsPluginModificationAllowed();
			});

			TestEqual(TEXT("Result matches config"), bResult, bExpected);
			if (bExpected)
			{
				ExpectNoException();
			}
			else
			{
				ExpectExceptionContains(TEXT("PluginToolset"));
			}
		});
	});

	Describe(TEXT("GetPluginDependencies"), [this]()
	{
		It(TEXT("Returns dependency entries for a discovered plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				// PluginToolset declares ToolsetRegistry and PluginUtils as dependencies
				const TArray<FPluginDependencyToolsetInfo> Deps =
					UPluginToolset::GetPluginDependencies(TEXT("PluginToolset"));
				const bool bHasToolsetRegistry = Deps.ContainsByPredicate(
					[](const FPluginDependencyToolsetInfo& D) { return D.Name == TEXT("ToolsetRegistry"); });
				TestTrue(TEXT("Contains ToolsetRegistry dependency"), bHasToolsetRegistry);
			});
			ExpectNoException();
		});

		It(TEXT("Raises error for nonexistent plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::GetPluginDependencies(TEXT("NonexistentXYZ12345"));
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});
	});

	Describe(TEXT("GetPluginDependents"), [this]()
	{
		It(TEXT("Returns plugins that depend on a given plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				// PluginToolset declares ToolsetRegistry as a dependency
				const TArray<FString> Dependents =
					UPluginToolset::GetPluginDependents(TEXT("ToolsetRegistry"));
				TestTrue(TEXT("Contains PluginToolset"), Dependents.Contains(TEXT("PluginToolset")));
			});
			ExpectNoException();
		});

		It(TEXT("Returns a sorted array"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Dependents =
					UPluginToolset::GetPluginDependents(TEXT("ToolsetRegistry"));
				for (int32 i = 1; i < Dependents.Num(); ++i)
				{
					TestTrue(TEXT("Sorted"), Dependents[i - 1] <= Dependents[i]);
				}
			});
			ExpectNoException();
		});

		It(TEXT("Raises error for nonexistent plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::GetPluginDependents(TEXT("NonexistentXYZ12345"));
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});
	});

	Describe(TEXT("GetPluginDescriptor"), [this]()
	{
		It(TEXT("Returns descriptor for a discovered plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const FPluginDescriptorToolsetInfo Info = UPluginToolset::GetPluginDescriptor(TEXT("PluginToolset"));
				TestFalse(TEXT("Has FriendlyName"), Info.FriendlyName.IsEmpty());
			});
			ExpectNoException();
		});

		It(TEXT("Descriptor fields match IPlugin::GetDescriptor()"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("ToolsetRegistry"));
				if (!TestTrue(TEXT("Plugin found"), Plugin.IsValid()))
				{
					return;
				}
				const FPluginDescriptor& Desc = Plugin->GetDescriptor();
				const FPluginDescriptorToolsetInfo Info = UPluginToolset::GetPluginDescriptor(TEXT("ToolsetRegistry"));
				TestEqual(TEXT("Description matches"), Info.Description, Desc.Description);
				TestEqual(TEXT("Version matches"), Info.Version, Desc.Version);
				TestEqual(TEXT("VersionName matches"), Info.VersionName, Desc.VersionName);
				TestEqual(TEXT("FriendlyName matches"), Info.FriendlyName, Desc.FriendlyName);
			});
			ExpectNoException();
		});

		It(TEXT("Raises error for nonexistent plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::GetPluginDescriptor(TEXT("NonexistentXYZ12345"));
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});
	});

	Describe(TEXT("UpdatePluginDescriptor"), [this]()
	{
		It(TEXT("Raises error for nonexistent plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::UpdatePluginDescriptor(TEXT("NonexistentXYZ12345"), FPluginDescriptorToolsetInfo{});
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});

		It(TEXT("Round-trips a descriptor field change"), [this]()
		{
			const TSharedPtr<IPlugin> PluginToolsetPlugin = IPluginManager::Get().FindPlugin(TEXT("PluginToolset"));
			if (PluginToolsetPlugin.IsValid())
			{
				if (FApp::IsEngineInstalled() && PluginToolsetPlugin->GetLoadedFrom() == EPluginLoadedFrom::Engine)
				{
					AddInfo(TEXT("PluginToolset is an engine plugin in an installed engine build. Skipping test."));
					ExceptionHandler.Reset();
					return;
				}
				if (FApp::IsInstalled() && PluginToolsetPlugin->GetType() != EPluginType::Mod)
				{
					AddInfo(TEXT("Cannot edit non-mod plugins in an installed build. Skipping test."));
					ExceptionHandler.Reset();
					return;
				}
			}

			FPluginDescriptorToolsetInfo Original;
			ExceptionHandler->CaptureErrorsIn([this, &Original]()
			{
				Original = UPluginToolset::GetPluginDescriptor(TEXT("PluginToolset"));
			});
			ExpectNoException();

			FPluginDescriptorToolsetInfo Modified = Original;
			Modified.Description = TEXT("PluginToolsetTestDescriptionXYZ12345");

			ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
			ExceptionHandler->CaptureErrorsIn([this, &Modified]()
			{
				UPluginToolset::UpdatePluginDescriptor(TEXT("PluginToolset"), Modified);
			});
			ExpectNoException();

			ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const FPluginDescriptorToolsetInfo Updated = UPluginToolset::GetPluginDescriptor(TEXT("PluginToolset"));
				TestEqual(TEXT("Description updated"), Updated.Description, FString(TEXT("PluginToolsetTestDescriptionXYZ12345")));
			});
			ExpectNoException();

			// Restore
			ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
			ExceptionHandler->CaptureErrorsIn([this, &Original]()
			{
				UPluginToolset::UpdatePluginDescriptor(TEXT("PluginToolset"), Original);
			});
			ExpectNoException();

			if (USourceControlHelpers::IsAvailable())
			{
				const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("PluginToolset"));
				if (Plugin.IsValid())
				{
					USourceControlHelpers::RevertFiles({ Plugin->GetDescriptorFileName() });
				}
			}
		});

		It(TEXT("No-op when descriptor is unchanged"), [this]()
		{
			const TSharedPtr<IPlugin> PluginToolsetPlugin = IPluginManager::Get().FindPlugin(TEXT("PluginToolset"));
			if (PluginToolsetPlugin.IsValid())
			{
				if (FApp::IsEngineInstalled() && PluginToolsetPlugin->GetLoadedFrom() == EPluginLoadedFrom::Engine)
				{
					AddInfo(TEXT("PluginToolset is an engine plugin in an installed engine build. Skipping test."));
					ExceptionHandler.Reset();
					return;
				}
				if (FApp::IsInstalled() && PluginToolsetPlugin->GetType() != EPluginType::Mod)
				{
					AddInfo(TEXT("Cannot edit non-mod plugins in an installed build. Skipping test."));
					ExceptionHandler.Reset();
					return;
				}
			}

			const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("PluginToolset"));
			if (!TestTrue(TEXT("Plugin found"), Plugin.IsValid()))
			{
				ExceptionHandler.Reset();
				return;
			}

			const FString DescriptorFileName = Plugin->GetDescriptorFileName();
			const FDateTime MtimeBefore = IFileManager::Get().GetTimeStamp(*DescriptorFileName);

			FPluginDescriptorToolsetInfo Original;
			ExceptionHandler->CaptureErrorsIn([this, &Original]()
			{
				Original = UPluginToolset::GetPluginDescriptor(TEXT("PluginToolset"));
			});
			ExpectNoException();

			ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
			ExceptionHandler->CaptureErrorsIn([this, &Original]()
			{
				UPluginToolset::UpdatePluginDescriptor(TEXT("PluginToolset"), Original);
			});
			ExpectNoException();

			const FDateTime MtimeAfter = IFileManager::Get().GetTimeStamp(*DescriptorFileName);
			TestEqual(TEXT("File not modified"), MtimeBefore, MtimeAfter);
		});
	});

	Describe(TEXT("AddPluginDependency"), [this]()
	{
		It(TEXT("Raises error for nonexistent plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::AddPluginDependency(TEXT("NonexistentXYZ12345"), TEXT("ToolsetRegistry"), false, true);
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});

	});

	Describe(TEXT("RemovePluginDependency"), [this]()
	{
		It(TEXT("Raises error for nonexistent plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::RemovePluginDependency(TEXT("NonexistentXYZ12345"), TEXT("SomeDep"));
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});

		It(TEXT("Raises error when dependency not found"), [this]()
		{
			const TSharedPtr<IPlugin> PluginToolsetPlugin = IPluginManager::Get().FindPlugin(TEXT("PluginToolset"));
			if (PluginToolsetPlugin.IsValid())
			{
				if (FApp::IsEngineInstalled() && PluginToolsetPlugin->GetLoadedFrom() == EPluginLoadedFrom::Engine)
				{
					AddInfo(TEXT("PluginToolset is an engine plugin in an installed engine build. Skipping test."));
					ExceptionHandler.Reset();
					return;
				}
				if (FApp::IsInstalled() && PluginToolsetPlugin->GetType() != EPluginType::Mod)
				{
					AddInfo(TEXT("Cannot edit non-mod plugins in an installed build. Skipping test."));
					ExceptionHandler.Reset();
					return;
				}
			}

			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::RemovePluginDependency(TEXT("PluginToolset"), TEXT("NonexistentDepXYZ12345"));
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});

		It(TEXT("Add/remove dependency round-trip"), [this]()
		{
			const TSharedPtr<IPlugin> PluginToolsetPlugin = IPluginManager::Get().FindPlugin(TEXT("PluginToolset"));
			if (PluginToolsetPlugin.IsValid())
			{
				if (FApp::IsEngineInstalled() && PluginToolsetPlugin->GetLoadedFrom() == EPluginLoadedFrom::Engine)
				{
					AddInfo(TEXT("PluginToolset is an engine plugin in an installed engine build. Skipping test."));
					ExceptionHandler.Reset();
					return;
				}
				if (FApp::IsInstalled() && PluginToolsetPlugin->GetType() != EPluginType::Mod)
				{
					AddInfo(TEXT("Cannot edit non-mod plugins in an installed build. Skipping test."));
					ExceptionHandler.Reset();
					return;
				}
			}

			// Add a dependency that is not already present
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::AddPluginDependency(TEXT("PluginToolset"), TEXT("DataflowCore"), true, false);
			});
			ExpectNoException();

			// Verify the dependency appears in the in-memory descriptor
			{
				const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("PluginToolset"));
				if (TestTrue(TEXT("Plugin found"), Plugin.IsValid()))
				{
					const bool bFound = Plugin->GetDescriptor().Plugins.ContainsByPredicate(
						[](const FPluginReferenceDescriptor& Ref) { return Ref.Name == TEXT("DataflowCore"); });
					TestTrue(TEXT("Dependency present"), bFound);
				}
			}

			// Remove it
			ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::RemovePluginDependency(TEXT("PluginToolset"), TEXT("DataflowCore"));
			});
			ExpectNoException();

			// Verify it is gone
			{
				const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("PluginToolset"));
				if (TestTrue(TEXT("Plugin found"), Plugin.IsValid()))
				{
					const bool bFound = Plugin->GetDescriptor().Plugins.ContainsByPredicate(
						[](const FPluginReferenceDescriptor& Ref) { return Ref.Name == TEXT("DataflowCore"); });
					TestFalse(TEXT("Dependency removed"), bFound);
				}
			}

			// Revert SCC changes to avoid leaving a checkout open
			if (USourceControlHelpers::IsAvailable())
			{
				const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("PluginToolset"));
				if (Plugin.IsValid())
				{
					USourceControlHelpers::RevertFiles({ Plugin->GetDescriptorFileName() });
				}
			}
		});
	});

	Describe(TEXT("SetPluginEnabled"), [this]()
	{
		It(TEXT("Raises error for nonexistent plugin"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::SetPluginEnabled(TEXT("NonexistentXYZ12345"), true);
			});
			ExpectExceptionContains(TEXT("PluginToolset"));
		});

		It(TEXT("Disables and re-enables an existing plugin"), [this]()
		{
			bool bModificationAllowed = true;
			GConfig->GetBool(TEXT("EditorSettings"), TEXT("bCanModifyPluginsFromBrowser"), bModificationAllowed, GEditorIni);
			if (!bModificationAllowed)
			{
				ExceptionHandler.Reset();
				return;
			}

			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::SetPluginEnabled(TEXT("ToolsetRegistry"), false);
			});
			ExpectNoException();

			ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UPluginToolset::SetPluginEnabled(TEXT("ToolsetRegistry"), true);
			});
			ExpectNoException();
		});
	});
}
#endif
